// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioChannelAgnosticCore : ModuleRules
{
	public AudioChannelAgnosticCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		// Disable warnings of Experimental API
		bValidateExperimentalApi = false;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
		
		// We want to use enum definitions in AudioMixer without an Engine dependency.
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"AudioMixer"
			}
		);
		
		// When the Editor builds all plugins, it will build with engine. In that case add a dependency on CoreUObject.
		// If we don't do this we will get linker errors for the Engine expecting to find stuff defined here.
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("CoreUObject");
		}
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"SignalProcessing",
			}
		);
	}
}