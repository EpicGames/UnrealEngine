// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "TypeFamily/ChannelTypeFamily.h"

class FAudioChannelAgnosticCoreModule final : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		Audio::RegisterCatBaseClassesWithChannelRegistry();
	}
	
	virtual void ShutdownModule() override
	{
		Audio::UnregisterCatBaseClassesWithChannelRegistry();
	}
};
    
IMPLEMENT_MODULE(FAudioChannelAgnosticCoreModule, AudioChannelAgnosticCore);