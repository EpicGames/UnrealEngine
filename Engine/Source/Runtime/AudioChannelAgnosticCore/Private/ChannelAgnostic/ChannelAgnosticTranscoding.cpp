// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChannelAgnostic/ChannelAgnosticTranscoding.h"

#include "Algo/Transform.h"
#include "DSP/Ambisonics.h"
#include "DSP/MultiMono.h"
#include "DSP/SphericalHarmonicCalculator.h"
#include "DSP/Vbap.h"
#include "TypeFamily/ChannelTypeFamily.h"

namespace Audio
{
	bool MakeGainMatrixFrom(const FDiscreteChannelTypeFamily& InSrc, const FDiscreteChannelTypeFamily& InDst, TArray<float>& OutMatrix);
	
	namespace ChannelAgnosticTranscoder
	{
		// Discrete to Discrete
		FTranscoder GetTranscoder(const FDiscrete& InSrcType, const FDiscrete& InDstType, const FParams& InParams)
		{
			// Exact match? We can just memcpy each channel.
			// TODO. in future these could be shared-ptrs from the main CAT memory block.
			if (&InDstType == &InSrcType)
			{
				const int32 NumChannels = InDstType.NumChannels();
				return [NumChannels](TArrayView<const float*> Src, TArrayView<float*> Dst, const int32 NumFrames)
					{
						for (int32 i = 0; i < NumChannels; ++i)
						{
							FMemory::Memcpy(Dst[i], Src[i], NumFrames * sizeof(float));
						}
					};
			}

			switch (InParams.TranscodeMethod)
			{
			case EChannelTranscodeMethod::ChannelDrop:
				{
					const TArray<FDiscreteChannelTypeFamily::FSpeaker> SrcOrder = InSrcType.GetSpeakerOrder();
					const TArray<FDiscreteChannelTypeFamily::FSpeaker> DstOrder = InDstType.GetSpeakerOrder();
					const int32 NumSrcChannels = InSrcType.NumChannels();
					const int32 NumDstChannels = InDstType.NumChannels();
					return [SrcOrder, DstOrder, NumSrcChannels, NumDstChannels](TArrayView<const float*> SrcChannels, TArrayView<float*> DstChannels, const int32 NumFrames)
						{
							checkSlow(SrcChannels.Num() == NumSrcChannels);
							checkSlow(DstChannels.Num() == NumDstChannels);
							// Copy everything destination wants, and nothing else.	
							for (int32 i = 0; i < NumDstChannels; ++i)
							{
								const FName DstSpeaker = DstOrder[i].ID;
								if (const int32 SrcChannelIndex = SrcOrder.IndexOfByPredicate([&](const FDiscreteChannelTypeFamily::FSpeaker& j) { return j.ID == DstSpeaker; });
									SrcChannelIndex != INDEX_NONE)
								{
									FMemory::Memcpy(DstChannels[i], SrcChannels[SrcChannelIndex], NumFrames * sizeof(float));
								}
								else
								{
									FMemory::Memzero(DstChannels[i], NumFrames * sizeof(float));	
								}
							}
						};
				}

			case EChannelTranscodeMethod::MixUpOrDown:
				{
					if (TArray<float> Gains; MakeGainMatrixFrom(InSrcType, InDstType, Gains))
					{
						return [MixGains = MoveTemp(Gains)](TArrayView<const float*> InSrc, TArrayView<float*> InDst, const int32 NumFrames)
							{
								MultiMonoMixUpOrDown(InSrc, InDst, NumFrames, MixGains);
							};					
					}
				}
			}
			// fail.
			return {};
		}

		// Discrete to Ambisonics
		FTranscoder GetTranscoder(const FDiscrete& InFromType, const FSoundfield& InToType, const FParams&)
		{
			const TArray<FDiscreteChannelTypeFamily::FSpeaker>& Speakers = InFromType.GetSpeakerOrder();
			TArray<TArray<float>> AllChannelGainArrays;
			AllChannelGainArrays.SetNum(InFromType.NumChannels());

			for (int32 i = 0; i < InFromType.NumChannels(); ++i)
			{
				const FDiscreteChannelTypeFamily::FSpeaker& Speaker = Speakers[i];
			
				TArray<float>& GainArray = AllChannelGainArrays[i];
				GainArray.SetNumZeroed(InToType.NumChannels());

				// Skip LFE.
				if (Speakers[i].ID == LexToName(ESpeakerShortNames::LFE))
				{
					continue;
				}
			
				const float AzimuthRads = FMath::DegreesToRadians(Speaker.AzimuthDegrees);
				const float ElevationRads = FMath::DegreesToRadians(Speaker.ElevationDegrees);
				FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(InToType.GetAmbisonicsOrder(), AzimuthRads, ElevationRads, GainArray);
				FSphericalHarmonicCalculator::NormalizeGains(GainArray);
			}
	
			return [Gains = MoveTemp(AllChannelGainArrays)](TArrayView<const float*> Src, TArrayView<float*> Dst, const int32 NumFrames)
				{
					const int32 NumSrcChannels = Src.Num();
					for( int32 i = 0; i < Dst.Num(); ++i)
					{
						FMemory::Memzero(Dst[i], sizeof(float) * NumFrames);
					}
					for (int32 SourceChannelIndex = 0; SourceChannelIndex < NumSrcChannels; ++SourceChannelIndex)
					{
						EncodeMonoAmbisonicMixIn(MakeArrayView(Src[SourceChannelIndex], NumFrames), Dst, Gains[SourceChannelIndex]);
					}
				};
		}
	
		// Discrete to Composite
		FTranscoder GetTranscoder(const FDiscrete& InFrom,	const FComposite&	InTo,	const FParams&)
		{
			return {}; // TBD.
		}
		
		// Ambisonics to Discrete
		FTranscoder GetTranscoder(const FSoundfield& InFromType, const FDiscrete& InToType, const FParams&)
		{
			TArray<TArray<float>> AllChannelGainArrays;
			AllChannelGainArrays.SetNum(InToType.NumChannels());
			const TArray<FDiscreteChannelTypeFamily::FSpeaker>& Order = InToType.GetSpeakerOrder();
			for (int32 i = 0; i < InToType.NumChannels(); ++i)
			{
				const FDiscreteChannelTypeFamily::FSpeaker& Speaker = Order[i];
			
				TArray<float>& GainArray = AllChannelGainArrays[i];
				GainArray.SetNumZeroed(InFromType.NumChannels());

				// Skip LFE.
				if (Order[i].ID == LexToName(ESpeakerShortNames::LFE))
				{
					continue;
				}
			
				check(GainArray.Num() == FSoundfieldChannelTypeFamily::OrderToNumChannels(InFromType.GetAmbisonicsOrder()));
				const float AzimuthRads = FMath::DegreesToRadians(Speaker.AzimuthDegrees);
				const float ElevationRads = FMath::DegreesToRadians(Speaker.ElevationDegrees);
				FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(InFromType.GetAmbisonicsOrder(), AzimuthRads, ElevationRads, GainArray);
				FSphericalHarmonicCalculator::NormalizeGains(GainArray);
			}
	
			return [NumSpeakers = InToType.NumChannels(), Gains = MoveTemp(AllChannelGainArrays)](TArrayView<const float*> Src, TArrayView<float*> Dst, const int32 NumFrames)
				{
					for (int32 i = 0; i < NumSpeakers; ++i)
					{
						TArrayView<float> DstChannel = MakeArrayView(Dst[i],NumFrames);
						FMemory::Memzero(DstChannel.GetData(), DstChannel.NumBytes()); // Decoder is MixIn.
						DecodeMonoAmbisonicMixIn(Src, DstChannel, Gains[i]);
					}
				};
		}

		// Ambisonics to Ambisonics
		FTranscoder GetTranscoder(const FSoundfield& InFromType,const FSoundfield& InToType, const FParams& InParams)
		{
			// Ambisonics to Ambisonics
			if (&InToType == &InFromType)
			{
				// Same, so memcpy.
				return [](TArrayView<const float*> InSrc, TArrayView<float*> InDst, const int32 NumFrames)
					{
						for (int32 i = 0; i < InSrc.Num(); ++i)
						{
							FMemory::Memcpy(InDst[i], InSrc[i], sizeof(float) * NumFrames);
						}
					};
			}

			const int32 NumChannelsToCopy = FMath::Min(InToType.NumChannels(), InFromType.NumChannels());
			return [NumChannelsToCopy](TArrayView<const float*> InSrc, TArrayView<float*> InDst, const int32 NumFrames)
				{
					checkSlow(InSrc.Num() >= NumChannelsToCopy);
					checkSlow(InDst.Num() >= NumChannelsToCopy);
					// Copy as many channels as destination will hold.
					for (int32 i = 0; i < NumChannelsToCopy; ++i)
					{
						FMemory::Memcpy(InDst[i], InSrc[i], sizeof(float) * NumFrames);
					}

					// Zero unwritten channels.
					for (int32 i = NumChannelsToCopy; i < InDst.Num(); ++i)
					{
						FMemory::Memzero(InDst[i], sizeof(float) * NumFrames);					
					}
				};
		}
		
		// Ambisonics to Composite
		FTranscoder GetTranscoder(const FSoundfield&	InFrom,	const FComposite&	InTo,	const FParams&)
		{
			return {};	// TBD.
		}
				
		static FTranscoder GetCompositeToX(const FComposite&	InFrom,	const FChannelTypeFamily&	InTo,	const FParams& InParams)
		{
			using FCoderInfo = TTuple<FComposite::FContainedType, FTranscoder>;
			TArray<FCoderInfo> Coders;
			Coders.Reserve(InFrom.NumTypes());
			for (int32 i = 0; i < InFrom.NumTypes(); ++i)
			{
				const FComposite::FContainedType* Contained = InFrom.GetType(i);
				const FChannelTypeFamily* ContainedFrom = GetChannelRegistry().FindChannel(Contained->Type);
				if (ensure(ContainedFrom))
				{
					Coders.Emplace(
						*Contained,
						FTranscoderResolver::Resolve(*ContainedFrom, InParams)
					);
				}
			}
			
			return [Coders](TArrayView<const float*> Src, TArrayView<float*> Dst, const int32 NumFrames)
				{	
					for (const FCoderInfo& i : Coders)
					{
						// Pull out stuff from tuple. 
						const int32 CompositeChannelOffset = i.Get<0>().Offset;
						const int32 CompositeNumChannels = i.Get<0>().Count;
						const FTranscoder& Coder = i.Get<1>();
						
						// Slice SRC info just what this composite item should see
						const TArrayView<const float*> SrcSlice = Src.Slice(CompositeChannelOffset, CompositeNumChannels); 
						
						// Execute.
						Coder(SrcSlice, Dst, NumFrames);
					}
				};
		}	
		
		// Composite to Soundfield
		FTranscoder GetTranscoder(const FComposite&	InFrom,	const FSoundfield&	InTo,	const FParams& InParams)
		{
			return GetCompositeToX(InFrom, InTo, InParams);
		}
		// Composite to Composite
		FTranscoder GetTranscoder(const FComposite&	InFrom,	const FComposite&	InTo,	const FParams& InParams)
		{
			return GetCompositeToX(InFrom, InTo, InParams);
		} 
		
		// Composite to Discrete
		FTranscoder GetTranscoder(const FComposite&	InFrom,	const FDiscrete&	InTo,	const FParams& InParams)
		{
			return GetCompositeToX(InFrom, InTo, InParams);			
		}
	
		static void TranslateGainMatrix(
			const TArrayView<float>& InGainMatrix,
			const int32 InNumInputChannels,
			const int32 InNumOutputChannels,
			const TArrayView<const ESpeakerShortNames>& InOldOrder,
			const TArrayView<const ESpeakerShortNames>& InNewOrder,
			TArray<float>& OutGains)
		{
			check(InOldOrder.Num() >= InNumInputChannels && InOldOrder.Num() >= InNumOutputChannels);
			// Channels not found in new order will be zeroed.
			OutGains.SetNumZeroed(InNumOutputChannels*InNumInputChannels);
		
			// Lookup (old->new).
			int32 Lookup[static_cast<int32>(ESpeakerShortNames::NumChannels)];
			for (int32& L : Lookup) 
			{ 
				L = INDEX_NONE; 
			}
			
			for (int32 i = 0; i < InOldOrder.Num(); ++i)
			{
				Lookup[static_cast<int32>(InOldOrder[i])] = InNewOrder.IndexOfByKey(InOldOrder[i]);
			}
		
			// Translate.
			for (int32 InputChannelIndex=0; InputChannelIndex < InNumInputChannels; ++InputChannelIndex)
			{
				// This input channel has a mapping in the new?
				const ESpeakerShortNames OldIn = InOldOrder[InputChannelIndex];
				if (const int32 NewIn = Lookup[static_cast<int32>(OldIn)]; NewIn != INDEX_NONE)
				{
					for (int32 OutputChannelIndex=0; OutputChannelIndex < InNumOutputChannels; ++OutputChannelIndex)
					{
						const ESpeakerShortNames OldOut = InOldOrder[OutputChannelIndex];
						if (const int32 NewOut = Lookup[static_cast<int32>(OldOut)]; NewOut != INDEX_NONE)
						{
							OutGains[(NewIn				* InNumOutputChannels)		+ NewOut] =
						InGainMatrix[(InputChannelIndex * InNumOutputChannels)		+ OutputChannelIndex];
						}
					}
				}
			}
		}
	
		static const IDiscretePanner* ChoosePanner(const FDiscreteChannelTypeFamily& InFrom, const FDiscreteChannelTypeFamily& InTo)
		{
			// Return To panner.
			// All using 2d vbap for now, so just return that.
			return InTo.GetPanner();
		}
	}
	
	// Static speakers can generate a matrix of speaker gains and cache that to avoid running the panner everytime.
	bool MakeGainMatrixFrom(const FDiscreteChannelTypeFamily& InSrc, const FDiscreteChannelTypeFamily& InDst, TArray<float>& OutMatrix)
	{
		// Example: Stereo -> Mono. 
		// Receiving VBAP, has only a single speaker FC and thus returns 100% gain.
		// IN   OUT
		//      FC 
		// FL  [ 0 ]
		// FR  [ 0 ]
				
		TSet<FName> MixedChannels;
		const int32 NumDstChannels = InDst.NumChannels();
		const int32 NumSrcChannels = InSrc.NumChannels();
		OutMatrix.SetNumZeroed(NumSrcChannels*NumDstChannels);										
		const TArray<FDiscreteChannelTypeFamily::FSpeaker>& SrcOrder = InSrc.GetSpeakerOrder();

		// See if there's a panner defined, use that.
		// Sometimes when there's no spatializable channels this will return false.
		if (const IDiscretePanner* Panner = ChannelAgnosticTranscoder::ChoosePanner(InSrc, InDst))
		{
			// For each src channel, create row vector of gains for each dst channel.
			for (int32 SrcIndex = 0; SrcIndex < NumSrcChannels; ++SrcIndex)
			{
				const FDiscreteChannelTypeFamily::FSpeaker& SrcSpeaker = SrcOrder[SrcIndex];
				const IDiscretePanner::FInputParams Params { 
					.AzimuthDegrees = SrcSpeaker.AzimuthDegrees, 
					.ElevationDegrees = SrcSpeaker.ElevationDegrees,
					.bAllowAzimuthMirroring = true
				};
				IDiscretePanner::FOutputParams Output;
				if (Panner->ComputeGains(Params,Output) )
				{
					const int32 RowOffset = SrcIndex * NumDstChannels;
					if (Output.Results.Num() > 0)
					{
						MixedChannels.Add(SrcSpeaker.ID);
					}

					// Copy results into gain array.
					for (int32 i = 0; i < Output.Results.Num(); ++i)
					{
						const IDiscretePanner::FPanResult& Result = Output.Results[i];
						if (const int32 DstChannelIndex = Result.ChannelIndex; DstChannelIndex != INDEX_NONE)
						{
							OutMatrix[RowOffset + DstChannelIndex] = Result.Gain;
						}
					}
				}
			}
		}
		
		// Non-spatialized channels will need to be copied over verbatim. (LFE, Centre).
		if (MixedChannels.Num() != InSrc.NumChannels())
		{
			for (int32 SrcIndex = 0; SrcIndex < NumSrcChannels; ++SrcIndex) 
			{
				const int32 RowOffset = SrcIndex * NumDstChannels;						
				const FDiscreteChannelTypeFamily::FSpeaker& SrcSpeaker = SrcOrder[SrcIndex];
				if (MixedChannels.Contains(SrcSpeaker.ID))
				{
					continue;
				}
			
				// Not spatialized at SOURCE and didn't mix it as part of the above panner. 
				if (!SrcSpeaker.bIsSpatialized)
				{
					// Find destination match or alias.
					if (const int32 IndexOf = InDst.GetSpeakerOrder().IndexOfByPredicate(
						[&SrcSpeaker](const FDiscreteChannelTypeFamily::FSpeaker& In) -> bool
							{
								return In.ID == SrcSpeaker.ID;
							}); IndexOf != INDEX_NONE)
					{
						OutMatrix[RowOffset + IndexOf] = 1.0f;	
						MixedChannels.Add(SrcSpeaker.ID);
					}
				}
			}
		}
		
		// A non-empty result is usable. 
		return !MixedChannels.IsEmpty();
	}
}