// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChannelAgnostic/ChannelAgnosticType.h"
#include "SimpleAlloc/SimpleHeapAllocator.h"
#include "TypeFamily/ChannelTypeFamily.h"

namespace Audio
{
	FSimpleAllocBase& FChannelAgnosticType::GetDefaultAllocator()
	{
		static FSimpleHeapAllocator Allocator;
		return Allocator;
	}

	FChannelAgnosticType::FChannelAgnosticType(const TRetainedRef<const FChannelTypeFamily> InType, const int32 InNumFrames, FSimpleAllocBase* InAllocator)
		: FChannelAgnosticType(InType, InNumFrames, InType.Get().NumChannels(), InAllocator)
	{
	}
	
	FChannelAgnosticType::FChannelAgnosticType(const TRetainedRef<const FChannelTypeFamily> InType, const int32 InNumFrames, const int32 InNumChannels, FSimpleAllocBase* InAllocator)
		: Buffer(InNumFrames * InNumChannels, InAllocator)
		, Type(&InType.Get()) // FChannelTypeFamily objects are registry-owned with static lifetime; raw pointer is safe and enables copyability.
		, NumFramesPrivate(InNumFrames)
		, NumChannelsPrivate(InNumChannels)
	{
		check(Type);
		check(NumChannelsPrivate > 0);
		check(Type->NumChannels() == InNumChannels || (Type->NumChannels() == 0 && InNumChannels > 0));
	}

	bool FChannelAgnosticType::IsA(const FChannelAgnosticType& InOther) const
	{
		check(Type);
		check(InOther.Type);
		return Type->IsA(InOther.Type);
	}

	bool FChannelAgnosticType::IsA(const FName& InTypeName) const
	{
		check(Type);
		return Type->IsA(InTypeName);
	}
	
	const FName& FChannelAgnosticType::GetTypeName() const
	{
		check(Type);
		return Type->GetName();
	}

	void FChannelAgnosticType::Zero()
	{
		FMemory::Memzero(Buffer.GetView().GetData(), Buffer.GetView().NumBytes());
	}
}
