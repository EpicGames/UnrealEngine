// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"

#include "Algo/Transform.h"
#include "ChannelAgnostic/ChannelAgnosticType.h"
#include "DSP/WavFormatUtils.h"

namespace Audio
{
	// TODO: MOVE to DSP.
	static void Interleave(const TArrayView<const float> InMultiMono, const int32 InNumChannels, const TArrayView<float> OutInterleaved)
	{
		check(InNumChannels > 0);
		checkSlow(InMultiMono.Num() <= OutInterleaved.Num());

		float* Dst = OutInterleaved.GetData();
		const float* Src = InMultiMono.GetData();
		const int32 NumFrames = InMultiMono.Num() / InNumChannels;
		checkSlow(InMultiMono.Num() % InNumChannels == 0);
					
		for(int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			for (int32 Channel = 0; Channel < InNumChannels; ++Channel)
			{
				*Dst++ = Src[NumFrames * Channel + Frame];
			}
		}
	}

	TOptional<uint32> FChannelAgnosticUtils::FindChannelMaskFromAudioMixerNumChannels(const int32 InNumChannels)
	{
		return WavFormatUtils::NumChannelsToCommonChannelMask(InNumChannels);
	}

	bool FChannelAgnosticUtils::ChannelIdToShortMixerChannels(const TArray<FName>& InIds, TArray<ESpeakerShortNames>& OutShortNames)
	{
		bool bResult = true;
		OutShortNames.Reset();
		Algo::Transform(InIds, OutShortNames, [&bResult](const FName InName) -> ESpeakerShortNames 
		{
			if (TOptional<ESpeakerShortNames> Enum = NameToShortSpeakerName(InName); Enum.IsSet())
			{
				return *Enum;
			}
			bResult = false;
			return ESpeakerShortNames::Unknown;
		});
		return bResult;	
	}
	

	void FChannelAgnosticUtils::Interleave(const FChannelAgnosticType& In, const TArrayView<float> Out)
	{
		Audio::Interleave(In.Buffer.GetView(), In.NumChannels(), Out);
	}

	TSharedPtr<const FDiscreteChannelTypeFamily> FChannelAgnosticUtils::FindDiscreteFormatFromNumChannels(const int32 InNumChannels)
	{
		const TOptional<uint32> Mask = FindChannelMaskFromAudioMixerNumChannels(InNumChannels);
		if (!Mask.IsSet())
		{
			return {};
		}
		return GetChannelRegistry().FindDiscreteChannelFromBitMask(*Mask);
	}

}