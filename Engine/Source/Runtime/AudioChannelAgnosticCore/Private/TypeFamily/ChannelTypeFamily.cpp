// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypeFamily/ChannelTypeFamily.h"

#include "Algo/Copy.h"
#include "HAL/FileManager.h"
#include "ChannelAgnostic/ChannelAgnosticTypeUtils.h"
#include "Containers/Map.h"
#include "DSP/SphericalHarmonicCalculator.h"
#include "DSP/Vbap.h"

namespace Audio
{
	namespace ChannelTypeFamilyPrivate
	{
		static FString MakePrettyString(const TArray<FDiscreteChannelTypeFamily::FSpeaker>& InSpeakers)
		{
			return FString::JoinBy(InSpeakers,TEXT(", "), [](FDiscreteChannelTypeFamily::FSpeaker InSpeaker) -> FString
			{
				return FString::Printf(TEXT("[Name=%s,Az=%2.2f, El=%2.2f]"), 
					*InSpeaker.ID.ToString(), InSpeaker.AzimuthDegrees, InSpeaker.ElevationDegrees);
			});
		}
	}
	
	
		
	class FChannelRegistryImpl final : public IChannelTypeRegistry 
	{
		TMap<FName, TSharedRef<const FChannelTypeFamily>> Types;
		TMap<uint32, TSharedRef<const FDiscreteChannelTypeFamily>> ChannelBitMaskToDiscreteLookupMap;		// Look up for discrete types from BitMask.
	public:
		UE_NONCOPYABLE(FChannelRegistryImpl);
		FChannelRegistryImpl() = default;
		virtual ~FChannelRegistryImpl() override = default;
	protected:
		virtual bool RegisterType(const FName& InUniqueName, TUniquePtr<FTypeFamily>&& InType) override
		{
			static const FName ChannelTypeName = TEXT("Cat");
			const FTypeFamily* Type = FindTypeInternal(ChannelTypeName);
			if (InUniqueName != ChannelTypeName)  // Protect the initial base type registration.
			{
				if (!Type)
				{
					// CAT not a registered type. Shouldn't happen.
					checkNoEntry();
					return false;
				}
			}
			
			if (Type && !InType->IsA(Type))
			{
				// Not a cat. (maybe a dog?)
				return false;	
			}
			
			// TODO: thread safety here.
			if (Types.Find(InUniqueName) != nullptr)
			{
				return false;
			}
			
			// Safe to cast it and add.
			TSharedRef<FChannelTypeFamily> Ref(static_cast<FChannelTypeFamily*>(InType.Release()));
			
			// Add to list of registered types.
			check(Types.Find(InUniqueName) == nullptr);
			Types.Emplace(InUniqueName,MoveTemp(Ref));
			
			// Rebuild lookups.
			BuildDiscreteBitmaskLookupMap();
			
			return true;
		}	
		virtual bool UnregisterType(const FName& InUniqueName) override
		{
			if (TSharedRef<const FChannelTypeFamily>* Found = Types.Find(InUniqueName))
			{
				const TSharedPtr<const FChannelTypeFamily> Removed(*Found);
				Types.Remove(InUniqueName);
				TSet<FName> Dependencies;
				for (auto i : Types)
				{
					if (i.Value->GetParent() && i.Value->GetParent() == Removed.Get())
					{
						Dependencies.Add(i.Key);
					}
				}
				
				// Clear deps.
				for (const FName i : Dependencies)
				{
					UnregisterType(i);
				}
				
				// Rebuild lookups.
				BuildDiscreteBitmaskLookupMap();
				return true;
			}
			return false;
		}
		
		virtual const FTypeFamily* FindTypeInternal(const FName InUniqueName) const override
		{
			if (const TSharedRef<const FChannelTypeFamily>* Found = Types.Find(InUniqueName))
			{
				return &(Found->Get());
			}
			return nullptr;
		}

		virtual TArray<TSharedRef<const FChannelTypeFamily>> GetAllChannelFormats() const override
		{
			TArray<TSharedRef<const FChannelTypeFamily>> AllFormats;
			Types.GenerateValueArray(AllFormats);
			return AllFormats;
		};
		
		virtual TArray<TSharedRef<const FChannelTypeFamily>> FindAllFamily(const FName InFamilyName) const override
		{
			TArray<TSharedRef<const FChannelTypeFamily>> Matches;
			const TArray<TSharedRef<const FChannelTypeFamily>> AllFormats = GetAllChannelFormats();
			Algo::CopyIf(AllFormats, Matches, [&InFamilyName](const TSharedRef<const FChannelTypeFamily>& i) -> bool
				{
					return i->GetFamilyName() == InFamilyName;
				});
			return Matches;
		}
		
		void BuildDiscreteBitmaskLookupMap()
		{
			ChannelBitMaskToDiscreteLookupMap.Reset();
			const TArray<TSharedRef<const FChannelTypeFamily>> AllDiscrete = FindAllFamily(FDiscreteChannelTypeFamily::GetFamilyTypeName());
			for (const TSharedRef<const FChannelTypeFamily>& i : AllDiscrete)
			{
				TSharedRef<const FDiscreteChannelTypeFamily> Discrete = StaticCastSharedRef<const FDiscreteChannelTypeFamily>(i);
				if (Discrete->IsAbstract())
				{
					continue;
				}

				// Assuming we can cleanly Convert FNames to Enum 
				if (TArray<ESpeakerShortNames> Channels; FChannelAgnosticUtils::ChannelIdToShortMixerChannels(Discrete->GetSpeakerIds(), Channels))
				{
					if (const uint32 BitMask = ShortNamesToChannelMask(Channels); BitMask != 0)
					{
						ensure(!ChannelBitMaskToDiscreteLookupMap.Contains(BitMask));
						ChannelBitMaskToDiscreteLookupMap.Emplace(BitMask, Discrete);
					}
				}
			}
		}
		virtual const TSharedPtr<const FDiscreteChannelTypeFamily> FindDiscreteChannelFromBitMask(const uint32 InChannelMask) const override
		{
			// Look up bitmask.
			if (const TSharedRef<const FDiscreteChannelTypeFamily>* Found = ChannelBitMaskToDiscreteLookupMap.Find(InChannelMask))
			{
				return Found->ToSharedPtr();
			}
	
			// Not found.
			return {};				
		}
	};

	int32 FSoundfieldChannelTypeFamily::OrderToNumChannels(const int32 InOrder)
	{
		return FSphericalHarmonicCalculator::OrderToNumChannels(InOrder);
	}

	TOptional<FChannelTypeFamily::FChannelName> FSoundfieldChannelTypeFamily::GetChannelName(const int32 InChannelIndex) const
	{
		if (InChannelIndex < 0)
		{
			return {};
		}

		int32 OrderNum=0;
		int32 DegreeNum=0;
		FSphericalHarmonicCalculator::IndexToOrderAndDegree(InChannelIndex, OrderNum, DegreeNum);
		const FName ChannelName = *FString::Printf(TEXT("ACN %d"), InChannelIndex); // *ChannelName.ToString(),	 (<-- Indexed' FName appears to fail for Node config UX, so use Sprintf'd one).ChannelName(TEXT("ACN"),InChannelIndex; 
		return FChannelName
		{
			.Name = ChannelName,  
			.FriendlyName = FString::Printf(TEXT("ACN=%d, Order=%d, Degree=%d"), InChannelIndex, Order, DegreeNum)
		};
	}
	

	IChannelTypeRegistry& GetChannelRegistry()
	{
		static FChannelRegistryImpl Registry;
		return Registry;
	}

   FChannelTypeFamily::FChannelTypeFamily(
	   const FName& InUniqueName,
	   const FName& InFamilyTypeName,
	   const int32 InNumChannels,
	   FChannelTypeFamily* InParentType,
	   const FString& InFriendlyName,
	   const bool InbIsParentsDefault,
	   const bool InbIsAbstract)
	   : Super(InUniqueName, InParentType, InFriendlyName)
	   , bIsAbstract(InbIsAbstract)
	   , bIsParentsDefault(InbIsParentsDefault)
	   , NumChannelsPrivate(InNumChannels)
	   , FamilyType(InFamilyTypeName)
	{
		check(InNumChannels >= 0);
		check(!InUniqueName.IsNone());
		if (bIsParentsDefault)
		{
			check(InParentType);
			check(InParentType->DefaultChild == nullptr); 
			InParentType->DefaultChild = this;
		}
	}

   FDiscreteChannelTypeFamily::FDiscreteChannelTypeFamily(const FName& InUniqueName, FChannelTypeFamily* InParentType, const FString& InFriendlyName, const TArray<FSpeaker>& InOrder, const bool bIsParentsDefault, const bool bIsAbstract)
		: Super(InUniqueName, GetFamilyTypeName(), InOrder.Num(), InParentType, InFriendlyName, bIsParentsDefault, bIsAbstract)
		, Order(InOrder)
		, Panner(MakePanner(InOrder))
	{
		checkf(InParentType, TEXT("Type=%s, Has a Null Parent"), *GetName().ToString());
	}

    FDiscreteChannelTypeFamily::~FDiscreteChannelTypeFamily()
	{
		// Needed for TUnique deleter, as panner is forward declared in header.
	}

   TOptional<FChannelTypeFamily::FChannelName> FDiscreteChannelTypeFamily::GetChannelName(const int32 InChannelIndex) const
	{
		check(InChannelIndex >= 0 && InChannelIndex < NumChannels());
		if (Order.IsValidIndex(InChannelIndex))
		{
			const FName SpeakerId = Order[InChannelIndex].ID;
			return
			{ // Optional.
					{ 
						.Name = SpeakerId,
						.FriendlyName = SpeakerId.ToString() 
					}
			};
		}
		return {};
	}

	TUniquePtr<IDiscretePanner> FDiscreteChannelTypeFamily::MakePanner(const TArray<FSpeaker>& InOrder)
	{
		// Just using 2D VBap for now.
		TArray<IDiscretePanner::FSpeaker> Speakers;
		for (int32 Channel = 0; Channel < InOrder.Num(); Channel++)
		{
			const FSpeaker& Speaker = InOrder[Channel];

			// Add all speakers that are considered 'spatialized'
			// This excludes channels that not, like LFE and possibly (Centre channel for dialog). 
			if (Speaker.bIsSpatialized)
			{
				IDiscretePanner::FSpeaker PannerSpeaker
				{
					.AzimuthDegrees = Speaker.AzimuthDegrees,
					.ElevationDegrees = Speaker.ElevationDegrees,
					.ChannelID = Speaker.ID,
					.ChannelIndex = Channel
				};
				Speakers.Add(PannerSpeaker);
			}
		}
		
		if (Speakers.Num() > 0)
		{
			return MakeUnique<FVBap2D>(Speakers);
		}
		return {};
	}
	
	
	void RegisterCatBaseClassesWithChannelRegistry(IChannelTypeRegistry& Registry)
	{
		// Register root type.
		Registry.RegisterType(TEXT("Cat"), MakeUnique<FChannelTypeFamily>(TEXT("Cat"), TEXT("Cat"),  0, nullptr, TEXT("Base Cat"), false, true));
		FChannelTypeFamily* BaseCat = Registry.FindChannel(TEXT("Cat"));
		check(BaseCat);

		Registry.RegisterType(
			 TEXT("Cat:Discrete"),
			MakeUnique<FDiscreteChannelTypeFamily>(
						TEXT("Cat:Discrete"),
						BaseCat,
						TEXT("Discrete Channel Format"),
						TArray<FDiscreteChannelTypeFamily::FSpeaker>(),
						true,
						true)
			);
		Registry.RegisterType(
			TEXT("Cat:Soundfield"),
			MakeUnique<FSoundfieldChannelTypeFamily>(
				TEXT("Cat:Soundfield"),
				0, BaseCat, TEXT("Soundfield Format"),
				false,
				true));
		
		Registry.RegisterType(
		TEXT("Cat:Composite"),
		MakeUnique<FCompositeChannelTypeFamily>(
			TEXT("Cat:Composite"),
			BaseCat,
			TEXT("Composite Format"),
			false));
	}
	
	void UnregisterCatBaseClassesWithChannelRegistry(IChannelTypeRegistry& Registry)
	{
		// Unregister C++ types and defaults. 
		Registry.UnregisterType(TEXT("Cat"));
		Registry.UnregisterType(TEXT("Cat:Discrete"));
		Registry.UnregisterType(TEXT("Cat:Soundfield"));
		Registry.UnregisterType(TEXT("Cat:Composite"));
	}
	
}


const TCHAR* LexToString(const ESpeakerShortNames InSpeaker)
{
	// No code gen support here, so role this manually.
#define CASE_AND_STRING(X) case ESpeakerShortNames::X: return TEXT(#X)

	using enum ESpeakerShortNames;
	switch (InSpeaker)
	{
	CASE_AND_STRING(FL);   // Front Left
	CASE_AND_STRING(FR);   // Front Right
	CASE_AND_STRING(FC);   // Front Center
	CASE_AND_STRING(LFE);  // Low-Frequency Effects (Subwoofer)
	CASE_AND_STRING(FLC);  // Front Left Center
	CASE_AND_STRING(FRC);  // Front Right Center
	CASE_AND_STRING(SL);   // Side Left
	CASE_AND_STRING(SR);   // Side Right
	CASE_AND_STRING(BL);   // Back Left
	CASE_AND_STRING(BR);   // Back Right
	CASE_AND_STRING(BC);   // Back Center
	CASE_AND_STRING(TFL);  // Top Front Left
	CASE_AND_STRING(TFR);  // Top Front Right
	CASE_AND_STRING(TBL);  // Top Back Left
	CASE_AND_STRING(TBR);  // Top Back Right
	default:
		break;
	}
	return TEXT("Unknown");
}
FName LexToName(const ESpeakerShortNames InSpeaker)
{
#define CASE_TO_CONST_FNAME(X)\
	case X:\
	{\
		static const FName Name = LexToString(InSpeaker);\
		return Name;\
	}
	switch (InSpeaker)
	{
		FOREACH_ENUM_ESPEAKERSHORTNAMES(CASE_TO_CONST_FNAME)
	default:
		checkNoEntry();
	}
	return {};
#undef CASE_TO_CONST_FNAME
}

TOptional<ESpeakerShortNames> NameToShortSpeakerName(const FName InName)
{
	#define IF_FNAME_RETURN(X)\
	{\
		static const FName Name = LexToString(X);\
		if (Name == InName) return X;\
	}
	FOREACH_ENUM_ESPEAKERSHORTNAMES(IF_FNAME_RETURN)
	#undef IF_FNAME_RETURN

	// failed.
	return {};
}

uint32 ShortNamesToChannelMask(const TArray<ESpeakerShortNames>& InChannels)
{
	uint32 ChannelMask = 0;
	for (const ESpeakerShortNames i : InChannels)
	{
		ChannelMask |= 1 << static_cast<uint32>(i);
	}
	return ChannelMask;
}
