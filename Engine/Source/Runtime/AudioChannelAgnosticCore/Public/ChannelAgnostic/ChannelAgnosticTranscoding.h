// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TypeFamily/ChannelTypeFamily.h"

namespace Audio
{   
	enum class EChannelTranscodeMethod : uint8
	{
		ChannelDrop,
		MixUpOrDown,
	};
	
	// All transcoder (from -> to) combinations.
	namespace ChannelAgnosticTranscoder 
	{
		struct FGetTranscoderParams
		{
			const FChannelTypeFamily& ToType;
			EChannelTranscodeMethod TranscodeMethod = EChannelTranscodeMethod::ChannelDrop;
			EChannelMapMonoUpmixMethod MixMethod = EChannelMapMonoUpmixMethod::EqualPower;
		};
				
		using FTranscoder = TFunction<void(TArrayView<const float*>, TArrayView<float*>, const int32 NumFrames)>;
		
		using FDiscrete = FDiscreteChannelTypeFamily;
		using FSoundfield = FSoundfieldChannelTypeFamily;
		using FComposite = FCompositeChannelTypeFamily;
		using FParams = FGetTranscoderParams;
		
		// Discrete-to-X
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FDiscrete&		InFrom,	const FDiscrete&	InTo,	const FParams&); // d->d
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FDiscrete&		InFrom,	const FSoundfield&	InTo,	const FParams&); // d->s
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FDiscrete&		InFrom,	const FComposite&	InTo,	const FParams&); // d->c
		
		// Soundfield-to-X
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FSoundfield&	InFrom,	const FSoundfield&	InTo,	const FParams&); // s->s
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FSoundfield&	InFrom,	const FDiscrete&	InTo,	const FParams&); // s->d
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FSoundfield&	InFrom,	const FComposite&	InTo,	const FParams&); // s->c
				
		// Composite-to-X
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FComposite&	InFrom,	const FComposite&	InTo,	const FParams&); // c->c
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FComposite&	InFrom,	const FDiscrete&	InTo,	const FParams&); // c->d
		AUDIOCHANNELAGNOSTICCORE_API FTranscoder GetTranscoder(const FComposite&	InFrom,	const FSoundfield&	InTo,	const FParams&); // c->s
	}
		
	template<typename TFromType>
	struct UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux") TFromVisitor final : IChannelTypeVisitor
	{
		const TFromType& FromType;
		const ChannelAgnosticTranscoder::FGetTranscoderParams& Params;
		ChannelAgnosticTranscoder::FTranscoder& Transcoder;
			
		TFromVisitor(const TFromType& InFromType, const ChannelAgnosticTranscoder::FGetTranscoderParams& InParams, ChannelAgnosticTranscoder::FTranscoder& InTranscoder)
			: FromType(InFromType), Params(InParams), Transcoder(InTranscoder)
		{}
			
		virtual void Visit(const FDiscreteChannelTypeFamily& InTo) override
		{
			Transcoder = ChannelAgnosticTranscoder::GetTranscoder(FromType, InTo, Params);	
		}
		virtual void Visit(const FSoundfieldChannelTypeFamily& InTo) override
		{
			Transcoder = ChannelAgnosticTranscoder::GetTranscoder(FromType, InTo, Params);
		}
		virtual void Visit(const FCompositeChannelTypeFamily& InTo) override
		{
			Transcoder = ChannelAgnosticTranscoder::GetTranscoder(FromType, InTo, Params);
		}
	};
		
	struct FTranscoderResolver final : public IChannelTypeVisitor
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		
		explicit FTranscoderResolver(const ChannelAgnosticTranscoder::FGetTranscoderParams& InParams) : Params(InParams) {}
		virtual void Visit(const FDiscreteChannelTypeFamily& InFrom) override
		{
			TFromVisitor Visitor(InFrom, Params, Result);
			Params.ToType.Accept(Visitor);
		}
		virtual void Visit(const FSoundfieldChannelTypeFamily& InFrom) override
		{
			TFromVisitor Visitor(InFrom, Params, Result);
			Params.ToType.Accept(Visitor);
		}
		virtual void Visit(const FCompositeChannelTypeFamily& InFrom) override
		{
			TFromVisitor Visitor(InFrom, Params, Result);
			Params.ToType.Accept(Visitor);	
		}
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		static ChannelAgnosticTranscoder::FTranscoder Resolve(const FChannelTypeFamily& FromType, const ChannelAgnosticTranscoder::FGetTranscoderParams& Params)
		{
			FTranscoderResolver Resolver(Params);
			FromType.Accept(Resolver);
			return MoveTemp(Resolver.Result);	
		}
		
		ChannelAgnosticTranscoder::FGetTranscoderParams Params;
		ChannelAgnosticTranscoder::FTranscoder Result;
		
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS		
	};
}