// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "ScratchBuffer.h"
#include "Templates/RetainedRef.h"
#include "TypeFamily/ChannelTypeFamily.h"

namespace Audio
{
	class FChannelTypeFamily;
	class FSimpleAllocBase;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	
	class FChannelAgnosticType
	{
	public:
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API static FSimpleAllocBase& GetDefaultAllocator();
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API explicit FChannelAgnosticType(const TRetainedRef<const FChannelTypeFamily> InType, const int32 InNumFrames, FSimpleAllocBase* InAllocator = &GetDefaultAllocator());

		/**
		 * 
		 * @param InType 
		 * @param InNumFrames 
		 * @param InNumChannels 
		 * @param InAllocator 
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API explicit FChannelAgnosticType(const TRetainedRef<const FChannelTypeFamily> InType, const int32 InNumFrames, const int32 InNumChannels, FSimpleAllocBase* InAllocator = &GetDefaultAllocator());
	
		// To be a Variable it *must* be copyable.
		FChannelAgnosticType(const FChannelAgnosticType& Other) = default;
		FChannelAgnosticType(FChannelAgnosticType&& Other) = default;
		FChannelAgnosticType& operator=(FChannelAgnosticType&& Other) = default;
		FChannelAgnosticType& operator=(const FChannelAgnosticType& Other) = default;

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		int32 NumFrames() const { return NumFramesPrivate; }

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		int32 NumChannels() const { return NumChannelsPrivate; }
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		TArrayView<const float> GetChannel(const int32 InChannelIndex) const
		{
			check(InChannelIndex >= 0 && InChannelIndex < NumChannelsPrivate);
			const TArrayView<const float> AllChannels = Buffer.GetView();
			return MakeArrayView(AllChannels.GetData() + (NumFramesPrivate * InChannelIndex), NumFramesPrivate);
		}
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		TArrayView<float> GetChannel(const int32 InChannelIndex)
		{
			const TArrayView<const float> ConstView = std::as_const(*this).GetChannel(InChannelIndex);
			return MakeArrayView<float>(const_cast<float*>(ConstView.GetData()), ConstView.Num());
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API bool IsA(const FChannelAgnosticType& InOther) const;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API bool IsA(const FName& InTypeName) const;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API const FName& GetTypeName() const;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FChannelTypeFamily& GetType() const { return *Type; }

		/*
		 * Zero the contents of the buffer.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API void Zero();

		/**
		 * For fast raw DSP access to the buffer.
		 * @return View of entire multi-mono buffer.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		TArrayView<float> GetRawMultiMono()
		{
			return Buffer.GetView();
		}
		/**
		 * For fast raw DSP access to the buffer.  
		 * @return View of entire multi-mono buffer.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]]
		TArrayView<const float> GetRawMultiMono() const
		{
			return Buffer.GetView();	
		}
		
	private:
		friend class FChannelAgnosticUtils;
		TScratchBuffer<float> Buffer;
		const FChannelTypeFamily* Type = nullptr;	// Keep this as pointer internally for easy of copying.
		int32 NumFramesPrivate = 0;
		int32 NumChannelsPrivate = 0;
	};

	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
}
