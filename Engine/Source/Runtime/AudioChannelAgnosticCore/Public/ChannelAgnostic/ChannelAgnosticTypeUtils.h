// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "TypeFamily/ChannelTypeFamily.h"

namespace Audio
{
	class FDiscreteChannelTypeFamily;
	class IChannelTypeRegistry;
	class FChannelAgnosticType;

	class FChannelAgnosticUtils
	{
	public:
		/**
		 * Attempts to match a ChannelCount to a ChannelMask, for compatibility with existing AudioMixer layouts.
		 * @return ChannelBitMask (Matching AudioMixerChannels enum shifted into a bitmask)
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")		
		AUDIOCHANNELAGNOSTICCORE_API static TOptional<uint32> FindChannelMaskFromAudioMixerNumChannels(const int32 InNumChannels);
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API static bool ChannelIdToShortMixerChannels(const TArray<FName>& InIds, TArray<ESpeakerShortNames>& OutShortNames);
				
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		static AUDIOCHANNELAGNOSTICCORE_API void Interleave(const FChannelAgnosticType& In, TArrayView<float> Out);

		/**
		 * Finds a discrete channel type family matching the given number of channels,
		 * using the standard AudioMixer channel layouts.
		 * @return Discrete channel type family, or null if no match found.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API static TSharedPtr<const FDiscreteChannelTypeFamily> FindDiscreteFormatFromNumChannels(const int32 InNumChannels);
	};
}
