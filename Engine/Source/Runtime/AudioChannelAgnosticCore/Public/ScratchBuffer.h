// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleAlloc/SimpleAllocBase.h"

namespace Audio
{
	template <typename T>
	class UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux") TScratchBuffer
	{
	public:
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		
		using ValueType = T;

		explicit TScratchBuffer(const int32 InNum, FSimpleAllocBase* InAllocator)
			: Allocator(InAllocator)
		{
			DoAllocation(InNum);
		}

		TScratchBuffer(const TScratchBuffer& Other)
			: Allocator(Other.Allocator)
		{
			*this = Other; // Call copy operator.
		}

		TScratchBuffer(TScratchBuffer&& Other)
			: Allocator(Other.Allocator)
		{
			*this = MoveTemp(Other); // Call move operator.
		}

		TScratchBuffer& operator=(const TScratchBuffer& Other)
		{
			if (this != &Other)
			{
				// Free our state.
				FreeAllocation();
				
				// Make new allocation.
				Allocator = Other.Allocator;
				DoAllocation(Other.Allocation.Num());

				// Copy other into new allocated memory 
				if (Allocation.GetData() && Allocation.Num() > 0)
				{
					FMemory::Memcpy(Allocation.GetData(), Other.Allocation.GetData(), Allocation.GetTypeSize() * Allocation.Num());
				}
			}
			return *this;
		}


		// Move operator.
		TScratchBuffer& operator=(TScratchBuffer&& Other)
		{
			if (this == &Other)
			{
				return *this;
			}
			// Free our state.
			FreeAllocation();
			Allocator = Other.Allocator;
			Allocation = MoveTemp(Other.Allocation);
			LifetimeToken = Other.LifetimeToken;

			// Reset just in case.
			Other.Reset();
			return *this;
		}

		~TScratchBuffer()
		{
			FreeAllocation();
		}

		bool IsValid() const
		{
			return Allocator->GetCurrentLifetime() == LifetimeToken;
		}

		int32 Num() const
		{
			check(IsValid());
			return Allocation.Num();
		}

		TArrayView<ValueType> GetView()
		{
			check(IsValid());
			return MakeArrayView(Allocation.GetData(), Num());
		}

		TArrayView<const ValueType> GetView() const
		{
			check(IsValid());
			return MakeArrayView(Allocation.GetData(), Num());
		}

	private:
		void Reset()
		{
			Allocation = {};
			LifetimeToken = -1;
		}

		void DoAllocation(const int32 InNum)
		{
			if (InNum > 0)
			{
				check(Allocation.GetData() == nullptr);
				Allocation = MakeArrayView<ValueType>(
					static_cast<ValueType*>(Allocator->Malloc(InNum * sizeof(ValueType))),
					InNum);
				check(Allocation.GetData() != nullptr);
				LifetimeToken = Allocator->GetCurrentLifetime();
				FMemory::Memzero(Allocation.GetData(), Allocation.GetTypeSize() * Allocation.Num());
			}
		}
		void FreeAllocation()
		{
			if (Allocation.GetData() != nullptr)
			{
				Allocator->Free(Allocation.GetData());
				Allocation = {};
			}
		}

		FSimpleAllocBase* Allocator = nullptr;
		TArrayView<ValueType> Allocation;
		int32 LifetimeToken = -1;
		
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	};
}