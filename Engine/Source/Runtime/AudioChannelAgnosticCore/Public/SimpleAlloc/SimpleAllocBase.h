// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"
#include "Misc/CoreMiscDefines.h"
#include "Math/UnrealMathUtility.h"

namespace Audio
{
	class FSimpleAllocBase
	{
	public:
		UE_NONCOPYABLE(FSimpleAllocBase);
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		FSimpleAllocBase() = default;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual ~FSimpleAllocBase() = default;
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		[[nodiscard]] virtual void* Malloc(const SIZE_T, const uint32 InAlignment = DEFAULT_ALIGNMENT) = 0;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Free(void*) = 0;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual uint32 GetCurrentLifetime() const { return 0; };
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Reset() {};
	};

	namespace SimpleAllocBasePrivate
	{
		// This is what's defined in MemoryBase, so follow it here. 
		FORCEINLINE static uint32 GetDefaultSizeToAlignment(const uint32 InSize)
		{
			check(InSize > 0);
			return InSize >= 16 ? 16 : 8;
		};
	}
}