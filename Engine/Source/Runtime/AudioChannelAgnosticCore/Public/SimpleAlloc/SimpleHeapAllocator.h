// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleAllocBase.h"

namespace Audio
{
	class FSimpleHeapAllocator final : public FSimpleAllocBase
	{
	public:
		UE_NONCOPYABLE(FSimpleHeapAllocator)
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		FSimpleHeapAllocator() = default;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void* Malloc(const SIZE_T InSizeBytes, const uint32 InAlignment=DEFAULT_ALIGNMENT) override
		{
			return FMemory::Malloc(InSizeBytes, InAlignment);
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Free(void* InPtr) override
		{
			return FMemory::Free(InPtr);
		}
	};
}
