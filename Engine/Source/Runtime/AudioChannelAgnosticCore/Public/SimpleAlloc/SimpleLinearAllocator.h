// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleAllocBase.h"
#include "Containers/ArrayView.h"

namespace Audio
{
	class FSimpleLinearAllocator : public FSimpleAllocBase
	{
	public:
		UE_NONCOPYABLE(FSimpleLinearAllocator)	
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		explicit FSimpleLinearAllocator(const TArrayView<uint8> InPage)
			: Page(InPage)
		{
			check(InPage.Num() > 0);
			check(IsAligned(InPage.GetData(), SimpleAllocBasePrivate::GetDefaultSizeToAlignment(InPage.Num())));
		}
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void* Malloc(const SIZE_T InSizeBytes, const uint32 InAlignment = DEFAULT_ALIGNMENT) override
		{
			check(InSizeBytes > 0);
			const uint32 Alignment = InAlignment == DEFAULT_ALIGNMENT ? SimpleAllocBasePrivate::GetDefaultSizeToAlignment(InSizeBytes) : InAlignment;
			const uint32 AlignedNewTop = Align(Top, Alignment);
		
			if (AlignedNewTop + InSizeBytes <= Page.Num())
			{
				Top = AlignedNewTop;
				void* Ret = Page.GetData() + Top;
				Top += InSizeBytes;
				return Ret;
			}
			return nullptr;
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Free(void*) override {}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual uint32 GetCurrentLifetime() const override
		{
			return CurrentLifetime;
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Reset() override
		{
			Top = 0;
			CurrentLifetime++;
		}
	
	protected:	
		uint32 Top = 0;					// Top of all allocations, climbs upwards towards end of page.
		uint32 CurrentLifetime = 0;		// Each reset the lifetime is increased
		TArrayView<uint8> Page;			// One page for now.
	};

	class FSimpleLinearAllocatorFromHeap final : public FSimpleLinearAllocator
	{
	public:
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		explicit FSimpleLinearAllocatorFromHeap(const SIZE_T InPageSize, const uint32 InPageAlignment = DEFAULT_ALIGNMENT)
			: FSimpleLinearAllocator(MakeArrayView(static_cast<uint8*>(FMemory::Malloc(InPageSize,InPageAlignment)), IntCastChecked<int32>(InPageSize)))
		{}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual ~FSimpleLinearAllocatorFromHeap() override
		{
			FMemory::Free(Page.GetData());
		}
	};
}
