// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "TypeFamily.h"
#include "Algo/Transform.h"
#include "DSP/ChannelMap.h"
#include "Templates/Function.h"
#include "Misc/Optional.h"


namespace Audio
{
	class FCompositeChannelTypeFamily;
	struct IDiscretePanner;
}

/**
 * A Comprehensive Short form Speaker enumeration
 */
enum class ESpeakerShortNames : uint32
{
	FL = EAudioMixerChannel::FrontLeft,				// Front Left
	FR = EAudioMixerChannel::FrontRight,			// Front Right
	FC = EAudioMixerChannel::FrontCenter,			// Front Center
	LFE = EAudioMixerChannel::LowFrequency,			// Low-Frequency Effects (Subwoofer)
	FLC = EAudioMixerChannel::FrontLeftOfCenter,	// Front Left Center
	FRC = EAudioMixerChannel::FrontRightOfCenter,	// Front Right Center
	SL = EAudioMixerChannel::SideLeft,				// Side Left
	SR = EAudioMixerChannel::SideRight,				// Side Right
	BL = EAudioMixerChannel::BackLeft,				// Back Left
	BR = EAudioMixerChannel::BackRight,				// Back Right
	BC = EAudioMixerChannel::BackCenter,			// Back Center
	TFL = EAudioMixerChannel::TopFrontLeft,			// Top Front Left
	TFR = EAudioMixerChannel::TopFrontRight,		// Top Front Right
	TBL = EAudioMixerChannel::TopBackLeft,			// Top Back Left
	TBR = EAudioMixerChannel::TopBackRight,			// Top Back Right

	NumChannels = EAudioMixerChannel::ChannelTypeCount,
	DefaultChannel = EAudioMixerChannel::DefaultChannel,
	Unknown = EAudioMixerChannel::Unknown
};

// Pasted from UHT generated.h, but we can't use because of AudioUnitTests.
#define FOREACH_ENUM_ESPEAKERSHORTNAMES(op) \
	op(ESpeakerShortNames::FL) \
	op(ESpeakerShortNames::FR) \
	op(ESpeakerShortNames::FC) \
	op(ESpeakerShortNames::LFE) \
	op(ESpeakerShortNames::FLC) \
	op(ESpeakerShortNames::FRC) \
	op(ESpeakerShortNames::SL) \
	op(ESpeakerShortNames::SR) \
	op(ESpeakerShortNames::BL) \
	op(ESpeakerShortNames::BR) \
	op(ESpeakerShortNames::BC) \
	op(ESpeakerShortNames::TFL) \
	op(ESpeakerShortNames::TFR) \
	op(ESpeakerShortNames::TBL) \
	op(ESpeakerShortNames::TBR)

AUDIOCHANNELAGNOSTICCORE_API const TCHAR* LexToString(const ESpeakerShortNames InSpeaker);
AUDIOCHANNELAGNOSTICCORE_API FName LexToName(const ESpeakerShortNames InSpeaker);
AUDIOCHANNELAGNOSTICCORE_API TOptional<ESpeakerShortNames> NameToShortSpeakerName(const FName InName);
AUDIOCHANNELAGNOSTICCORE_API uint32 ShortNamesToChannelMask(const TArray<ESpeakerShortNames>& InChannels);

namespace Audio
{
	// Forward declare these for visitor.
	class FDiscreteChannelTypeFamily;
	class FSoundfieldChannelTypeFamily;
	class FPackedChannelTypeFamily;
	
	class IChannelTypeVisitor
	{
	public:
		virtual ~IChannelTypeVisitor() = default;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Visit(const FDiscreteChannelTypeFamily&) = 0;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Visit(const FSoundfieldChannelTypeFamily&) = 0;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Visit(const FCompositeChannelTypeFamily&) = 0;
	};
	
	// Base type for all "channel" based types.
	class FChannelTypeFamily : public FTypeFamily
	{
	public:
		UE_NONCOPYABLE(FChannelTypeFamily);
		using Super = FTypeFamily;

		/**
		 * Constructor. 
		 * @param InUniqueName The unique name that will be used for look up in registry.
		 * @param InFamilyTypeName The name of the concrete type (c++) that's defining this. (for safe casting). 
		 * @param InNumChannels Num of channels in this type. (pure categorical, organisational entries will be 0).
		 * @param InParentType Pointer to this parents type in the tree. Can be null.
		 * @param InFriendlyName Friendly name to display to the user e.g. "Dolby Stereo (2.0)"
		 * @param InbIsParentsDefault Marks if this entry is the default child of its parent. 
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API explicit FChannelTypeFamily(
			const FName& InUniqueName,
			const FName& InFamilyTypeName,
			const int32 InNumChannels,
			FChannelTypeFamily* InParentType,
			const FString& InFriendlyName,
			const bool InbIsParentsDefault,
			const bool InbIsAbstract);

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual ~FChannelTypeFamily() = default;

		/**
		 * If this type can be instantiated or is just organisational.
		 * @return true if abstract, false otherwise
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool IsAbstract() const
		{
			return bIsAbstract;
		}

		/**
		 * If this is marked as being the default on its parent
		 * @return true if true, false otherwise.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool IsParentsDefault() const
		{
			return bIsParentsDefault;
		}

		/**
		 * Returns the default child if one exists. Example. "Stereo" would return "Stereo_2_0"
		 * @return Pointer to type, null otherwise.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FChannelTypeFamily* GetDefaultChild() const { return DefaultChild; }

		/**
		 * Num of channels  
		 * @return >= 0
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		int32 NumChannels() const { return NumChannelsPrivate; }

		using IPanner = IDiscretePanner;
		
		struct FGetPannerParams
		{
			// Control which algorithm to use for this format.
			// Not currently used.
			// FName PannerType = TEXT("VBAP 2D");
		};
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual const IPanner* GetPanner(const FGetPannerParams& InParams = {}) const { return nullptr; }
		
		/**
		 * Returns the family name (C++ type) of this ChannelType. This allows us to safely downcast (if necessary).
		 * @return The name of this type Family. (i.e. Discrete/Ambisoncs). 
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		FName GetFamilyName() const { return FamilyType; }

		/**
		 * Returns the name of the channel. For discrete this would be what speaker its mapped to etc.
		 * @param InChannelIndex Index of channel >= 0 and < NumChannels.
		 * @return Optional containing the Channel name and friendly name.
		 */
		struct FChannelName
		{
			FName Name;
			FString FriendlyName;
		};

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual TOptional<FChannelName> GetChannelName(const int32 InChannelIndex) const { return {}; }

		template<typename T>
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		T* Cast()
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			if (T::GetFamilyTypeName() == GetFamilyName())
			{
				return static_cast<T*>(this);
			}
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
			return nullptr;
		}

		template<typename T>
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const T* Cast() const { return const_cast<FChannelTypeFamily*>(this)->Cast<T>(); }

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Accept(IChannelTypeVisitor& InVisitor) const
		{
			checkNoEntry();
		};

	protected:
		// Both are friends so we can hide the api.
		friend class FDiscreteChannelTypeFamily;
		friend class FSoundfieldChannelTypeFamily;

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		void SetNumChannels(const int32 InNumChannels)
		{
			NumChannelsPrivate = InNumChannels;
		}
		
	private:
		bool bIsAbstract = false;
		bool bIsParentsDefault = false;
		const FChannelTypeFamily* DefaultChild = nullptr;
		int32 NumChannelsPrivate = 0;
		FName FamilyType; 
	};
	
	class FDiscreteChannelTypeFamily : public FChannelTypeFamily 
	{
	public:
		UE_NONCOPYABLE(FDiscreteChannelTypeFamily);
		using Super = FChannelTypeFamily;

		struct FSpeaker
		{
			FName ID;								// ID of speaker.
			float AzimuthDegrees = 0.f;				// -180 (left), 0 (front), 180 (right).
			float ElevationDegrees = 0.f;			// -180 (down), 0 (ground), 180 (up). 
			bool bIsSpatialized = true;				// If false, azimuth+elevation are ignored.
		};
		
		static FName GetFamilyTypeName()
		{
			static const FName Name = TEXT("Discrete");
			return Name;
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API FDiscreteChannelTypeFamily(const FName& InUniqueName, FChannelTypeFamily* InParentType, const FString& InFriendlyName,
			const TArray<FSpeaker>& InOrder, const bool bIsParentsDefault, const bool bIsAbstract);

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API virtual ~FDiscreteChannelTypeFamily() override;
		
		/**
		 * Returns true if this format defines any height data. 
		 * @return true if heights are found.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool HasHeightChannels() const
		{
			return Order.FindByPredicate([](const FSpeaker& i) -> bool { return !FMath::IsNearlyZero(i.ElevationDegrees); }) != nullptr;
		}
		
		/**
		 * Find the index in the list of Channels for speaker.
		 * @param InSpeaker 
		 * @return >= 0 (index) INDEX_NONE (failed to find).
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		int32 FindSpeakerIndex(const ESpeakerShortNames InSpeaker) const
		{
			const FName SpeakerName = LexToString(InSpeaker);
			return Order.IndexOfByPredicate([SpeakerName](const FSpeaker& i) -> bool { return SpeakerName == i.ID; });
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		int32 FindSpeakerIndex(const FName InSpeaker) const
		{
			return Order.IndexOfByPredicate([InSpeaker](const FSpeaker& i) -> bool { return InSpeaker == i.ID; });
		}
		

		/**
		 * Checks if a paticular speaker is present in the Channels for this Format.
		 * @param InSpeaker 
		 * @return true (exists), false otherwise.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool HasSpeaker(const ESpeakerShortNames InSpeaker) const
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			return FindSpeakerIndex(InSpeaker) != INDEX_NONE;
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}

		/**
		 * Get the function used for translating/transcoding between this and the "To" type.
		 * @param InToType The Type we are transcoding into.
		 * @return The Function Object. (this will be not be set if a function wasn't returned)
		 */	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API virtual TOptional<FChannelName> GetChannelName(const int32 InChannelIndex) const override;

		/**
		 * Get the order of the speakers defined on this discrete format.
		 * @return Array of speakers in order.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const TArray<FSpeaker>& GetSpeakerOrder() const { return Order; }
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		TArray<FName> GetSpeakerIds() const
		{
			TArray<FName> Ids;
			Algo::Transform(Order, Ids, [](const FSpeaker& i) -> FName { return i.ID; });
			return Ids;
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Accept(IChannelTypeVisitor& InVisitor) const override
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			InVisitor.Visit(*this);
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual const IPanner* GetPanner(const FGetPannerParams& InParams ={}) const override
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			return Panner.Get();	
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}

	protected:
		static TUniquePtr<IDiscretePanner> MakePanner(const TArray<FSpeaker>& Order);
		TArray<FSpeaker> Order;
		TUniquePtr<IPanner> Panner;	// Just one for now.
	};

	// Soundfield
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	class FSoundfieldChannelTypeFamily : public FChannelTypeFamily
	{
	public:
		UE_NONCOPYABLE(FSoundfieldChannelTypeFamily);
		using Super = FChannelTypeFamily;


		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		static FName GetFamilyTypeName()
		{
			static const FName Name = TEXT("Soundfield");
			return Name;
		}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API static int32 OrderToNumChannels(const int32 InOrder);

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		explicit FSoundfieldChannelTypeFamily(const FName& InUniqueName, const int32 InOrder, FChannelTypeFamily* InParentType, const FString& InFriendlyName, const bool bIsParentsDefault, const bool InbIsAbstract)
			: Super(InUniqueName, GetFamilyTypeName(), OrderToNumChannels(InOrder), InParentType, InFriendlyName, bIsParentsDefault, InbIsAbstract)
			, Order(InOrder)
		{}

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		int32 GetAmbisonicsOrder() const { return Order; }
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		AUDIOCHANNELAGNOSTICCORE_API virtual TOptional<FChannelName> GetChannelName(const int32 InChannelIndex) const override;

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Accept(IChannelTypeVisitor& InVisitor) const override
		{
			InVisitor.Visit(*this);
		}
		
	private:
		int32 Order = 0;
	};
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	class FCompositeChannelTypeFamily : public FChannelTypeFamily
	{
	public:
		UE_NONCOPYABLE(FCompositeChannelTypeFamily);

		using Super = FChannelTypeFamily;


		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		static FName GetFamilyTypeName()
		{
			static const FName Name = TEXT("Composite");
			return Name;
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		explicit FCompositeChannelTypeFamily(const FName& InUniqueName, FChannelTypeFamily* InParentType, const FString& InFriendlyName, const bool bIsParentsDefault)
			: Super(InUniqueName, GetFamilyTypeName(), /* Placeholder, NumChannels=1 */ 1, InParentType, InFriendlyName, bIsParentsDefault, false)
		{
		}
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		int32 NumTypes() const
		{
			return Types.Num();
		}

		struct FContainedType
		{
			FName Type;					// Type name of embedded format.
			uint32 Offset = 0;			// Index in channels.	
			uint32 Count = 0;			// Count in channels.
		};		
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")		
		const FContainedType* GetType(const int32 InTypeIndex) const
		{
			if (Types.IsValidIndex(InTypeIndex))
			{
				return &Types[InTypeIndex];
			}
			return nullptr;
		}
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual void Accept(IChannelTypeVisitor& InVisitor) const override
		{
			InVisitor.Visit(*this);
		}
		
	protected:
		TArray<FContainedType> Types;
	};
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS


	// Channel type registry.
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	class IChannelTypeRegistry : public ITypeFamilyRegistry
	{
	public:
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		const FChannelTypeFamily* FindChannel(const FName InName) const
		{
			return Find<FChannelTypeFamily>(InName);
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		FChannelTypeFamily* FindChannel(const FName InName)
		{
			return const_cast<FChannelTypeFamily*>(const_cast<const IChannelTypeRegistry*>(this)->FindChannel(InName));
		}
	
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		virtual const TSharedPtr<const FDiscreteChannelTypeFamily> FindDiscreteChannelFromBitMask(const uint32 InChannelMask) const = 0;

		/**
		 * Find the first concrete (non-abstract, > 0 channels) channel. (e.g. Stereo -> 'Stereo_2_0')
		 * Each Abstract type should have a designated default child.
		 *
		 * @param InName Name of format.
		 * @return Pointer to type (on success), nullptr on failure
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		const FChannelTypeFamily* FindConcreteChannel(const FName InName) const
		{
			check(!InName.IsNone());

			// Walk looking for non-abstract default child. (e.g. Surround->Surround5X->Surround 5.1)
			const FChannelTypeFamily* Channel = FindChannel(InName);
			while (Channel && Channel->IsAbstract())
			{
				Channel = Channel->GetDefaultChild();
			}
			return Channel;
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		const FChannelTypeFamily& FindConcreteChannelChecked(const FName InName) const
		{
			const FChannelTypeFamily* Concrete = FindConcreteChannel(InName);
			checkf(Concrete, TEXT("Concrete Channel not Found: %s"), *InName.ToString());
			return *Concrete;
		}

		/**
		 * Returns every registered format as an array.
		 * @return Array of format types.
		 */
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")			
		virtual TArray<TSharedRef<const FChannelTypeFamily>> GetAllChannelFormats() const = 0;
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")	
		virtual TArray<TSharedRef<const FChannelTypeFamily>> FindAllFamily(const FName InFamilyName) const = 0;		

	};
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
	AUDIOCHANNELAGNOSTICCORE_API IChannelTypeRegistry& GetChannelRegistry();
	
	UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")	
	AUDIOCHANNELAGNOSTICCORE_API void RegisterCatBaseClassesWithChannelRegistry(IChannelTypeRegistry& Registry = GetChannelRegistry());
	
	UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")		
	AUDIOCHANNELAGNOSTICCORE_API void UnregisterCatBaseClassesWithChannelRegistry(IChannelTypeRegistry& Registry = GetChannelRegistry());

}//namespace Audio
