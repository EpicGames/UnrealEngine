// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/RetainedRef.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

namespace Audio
{
	class FTypeFamily
	{
		UE_NONCOPYABLE(FTypeFamily); 
		
		FName Name;
		const FTypeFamily* Parent = nullptr;
		FString FriendlyName;
		
		template<typename Predicate> 
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FTypeFamily* FindAncestor(Predicate Pred) const
		{
			for (const FTypeFamily* i=this; i; i=i->Parent)
			{
				if (::Invoke(Pred,i))
				{
					return i;
				}
			}
			return nullptr;
		}
	public:
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		FTypeFamily(const FName InName, const FTypeFamily* InParent, const FString& InFriendlyName)
			: Name(InName)
			, Parent(InParent)
			, FriendlyName(InFriendlyName)
		{}
		
		virtual ~FTypeFamily() = default;

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool IsA(const FTypeFamily* Other) const
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS			
			return nullptr != FindAncestor([Other](const FTypeFamily* i) -> bool { return i==Other; });
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS			
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		bool IsA(const FName& Other) const
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS			
			return nullptr != FindAncestor([&](const FTypeFamily* i) -> bool { return i->Name==Other; });
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS			
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FName& GetName() const { return Name; }
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FString& GetFriendlyName() const { return FriendlyName; }
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const FTypeFamily* GetParent() const { return Parent; }
	};

	// Simple type registry.
	class ITypeFamilyRegistry
	{
	public:
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual ~ITypeFamilyRegistry() = default;

		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual bool RegisterType(const FName& InUniqueName, TUniquePtr<FTypeFamily>&& InType) = 0;
		
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual bool UnregisterType(const FName& InUniqueName) = 0;

		template<typename T> 
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const T* Find(const FName& InUniqueName) const
		{
			// TODO: test if this is valid by using FName Comparison.	
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS			
			return static_cast<const T*>(FindTypeInternal(InUniqueName));
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS			
		}
	protected:
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		virtual const FTypeFamily* FindTypeInternal(const FName InUniqueName) const = 0;
	};

	// Convenience template so we don't have to specialize each Find. 
	template<typename T> struct TFamilyRegistry
	{
		ITypeFamilyRegistry& FamilyRegistry;
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		explicit TFamilyRegistry(ITypeFamilyRegistry& InFamilyRegistry)
			: FamilyRegistry(InFamilyRegistry)
		{}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const T* Find(const FName& InName) const
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS			
			return FamilyRegistry.Find<T>(InName);
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS			
		}
		UE_EXPERIMENTAL(5.8,"AudioChannelAgnosticCore is experimental and in flux")
		const T& FindChecked(const FName& InName) const
		{
			PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS			
			const T* Found = Find(InName);
			check(Found);
			return *Found;
			PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS			
		}
	};
}